<?xml version="1.0" ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../../Engine/Build/Graph/Schema.xsd" >
	
	<Property Name="BaseDir" Value="Engine/Source/Programs/OidcToken" />
	<Property Name="BaseOutputPath" Value="Engine/Binaries/DotNET/OidcToken/" />
	
	<Agent Name="OidcToken Win64" Type="Win64">
		<Node Name="Test OidcToken Win64">
			<DotNet BaseDir="$(BaseDir)" Arguments="test" />
		</Node>
		
		<Property Name="OutputPath" Value="$(BaseOutputPath)/win-x64/" />
		
		<Node Name="Build OidcToken Win64" Requires="Test OidcToken Win64">
			<DotNet BaseDir="$(BaseDir)" Arguments="publish -p:PublishProfile=PublishWindows" />
		</Node>
		
		<Node Name="Sign OidcToken Win64" Requires="Build OidcToken Win64">
			<Sign Files="$(OutputPath)/OidcToken.exe" />
		</Node>

		<Node Name="Tag OutputFilesWin64" Requires="Sign OidcToken Win64" Produces="#OutputFilesWin64">
			<Tag Files="$(OutputPath)/..." With="#OutputFilesWin64" />
		</Node>
	</Agent>
	
	<Agent Name="OidcToken Mac" Type="Mac">
		<Node Name="Test OidcToken Mac">
			<DotNet BaseDir="$(BaseDir)" Arguments="test" />
		</Node>

		<Property Name="OutputPath" Value="$(BaseOutputPath)/osx-x64/" />
		
		<Node Name="Build OidcToken Mac" Requires="Test OidcToken Mac">
			<DotNet BaseDir="$(BaseDir)" Arguments="publish -p:PublishProfile=PublishMac" />
		</Node>

		<Node Name="Sign OidcToken Mac" Requires="Build OidcToken Mac">
			<Sign Files="$(OutputPath)/OidcToken" />
		</Node>

		<Node Name="Tag OutputFilesMac" Requires="Sign OidcToken Mac" Produces="#OutputFilesMac">
			<Tag Files="$(OutputPath)/..." With="#OutputFilesMac" />
		</Node>
	</Agent>

	<Agent Name="OidcToken Linux" Type="Linux">
		<Node Name="Test OidcToken Linux">
			<DotNet BaseDir="$(BaseDir)" Arguments="test" />
		</Node>

		<Property Name="OutputPath" Value="$(BaseOutputPath)/linux-x64/" />

		<Node Name="Build OidcToken Linux" Requires="Test OidcToken Linux">
			<DotNet BaseDir="$(BaseDir)" Arguments="publish -p:PublishProfile=PublishLinux" />
		</Node>
		
		<!-- No code signing for Linux -->

		<Node Name="Tag OutputFilesLinux" Requires="Build OidcToken Linux" Produces="#OutputFilesLinux">
			<Tag Files="$(OutputPath)/..." With="#OutputFilesLinux" />
		</Node>
	</Agent>

	<Agent Name="OidcToken Portable" Type="Win64">
		<Node Name="Test OidcToken Portable">
			<DotNet BaseDir="$(BaseDir)" Arguments="test" />
		</Node>

		<Property Name="OutputPath" Value="$(BaseOutputPath)/portable/" />

		<Node Name="Build OidcToken Portable" Requires="Test OidcToken Portable">
			<DotNet BaseDir="$(BaseDir)" Arguments="publish -p:PublishProfile=PublishPortable" />
		</Node>

		<!-- No code signing for Portable -->

		<Node Name="Tag OutputFilesPortable" Requires="Build OidcToken Portable" Produces="#OutputFilesPortable">
			<Tag Files="$(OutputPath)/..." With="#OutputFilesPortable" />
		</Node>
	</Agent>
	
	<Agent Name="Submit OidcToken" Type="Win64">
		<Node Name="Tag OutputFiles" Requires="#OutputFilesWin64;#OutputFilesMac;#OutputFilesLinux;#OutputFilesPortable" Produces="#OutputFiles">
			<Tag With="#OutputFiles" Files="#OutputFilesWin64;#OutputFilesMac;#OutputFilesLinux;#OutputFilesPortable" />
		</Node>
		
		<Node Name="Publish OidcToken" Requires="#OutputFiles">
			<Submit Description="Updating OidcToken from CL $(Change)\n\n#okforgithub public" Files="#OutputFiles" RevertUnchanged="true" />
		</Node>
	</Agent>
</BuildGraph>
