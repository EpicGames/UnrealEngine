<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <ImplicitUsings>enable</ImplicitUsings>
    <Nullable>enable</Nullable>

    <IsPackable>false</IsPackable>
    <IsTestProject>true</IsTestProject>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="coverlet.collector" Version="8.0.1" />
    <PackageReference Include="Microsoft.NET.Test.Sdk" Version="18.4.0" />
    <PackageReference Include="Moq" Version="4.20.72" />
    <PackageReference Include="Moq.Contrib.HttpClient" Version="1.4.0" />
    <PackageReference Include="MSTest.TestAdapter" Version="4.2.1" />
    <PackageReference Include="MSTest.TestFramework" Version="4.2.1" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\OidcToken.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Using Include="Microsoft.VisualStudio.TestTools.UnitTesting" />
  </ItemGroup>

</Project>
