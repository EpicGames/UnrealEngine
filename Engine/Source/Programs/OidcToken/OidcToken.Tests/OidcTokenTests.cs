// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Net.Mime;
using System.Text;
using System.Text.Json;
using EpicGames.OIDC;
using IdentityModel.Client;
using IdentityModel.OidcClient;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Moq;
using Serilog;

namespace OidcToken.Tests
{
	[TestClass]
	public class OidcTokenTests
	{
		private IHost SetupMocks(Action<TokenServiceOptions> configureCliOptions, Action<OidcTokenOptions> configureOidcOptions, Func<OidcTokenClient> mockTokenClient, ITokenStore tokenStore)
		{
			Mock<IOidcTokenClientFactory> clientFactory = new Mock<IOidcTokenClientFactory>();
			clientFactory.Setup(factory =>  factory.CreateTokenClient(It.IsAny<string>(), It.IsAny<ProviderInfo>(), It.IsAny<TimeSpan>())).Returns(mockTokenClient);

			IHost host = Host.CreateDefaultBuilder()
				.UseEnvironment("Testing")
				.ConfigureServices(collection => collection.AddSerilog())
				.ConfigureServices(
					(content, services) =>
					{
						services.AddOptions<TokenServiceOptions>().Configure(configureCliOptions).ValidateDataAnnotations();
						services.AddOptions<OidcTokenOptions>().Configure(configureOidcOptions).ValidateDataAnnotations();

						services.AddSingleton<IOidcTokenManager, OidcTokenManager>();
						services.AddSingleton<IOidcTokenClientFactory>(clientFactory.Object);
						services.AddSingleton<ITokenStore>(tokenStore);

						services.AddSingleton<TokenService>();
						services.AddHostedService<TokenService>(provider => provider.GetService<TokenService>()!);
					})
					.Build();

			return host;
		}

		[TestMethod]
		public async Task AllocateTokenToJsonFile()
		{
			const string providerName = "Test-Service";
			ProviderInfo providerInfo = new ProviderInfo()
			{
				ClientId = "test-client",
				DisplayName = "test-client",
				RedirectUri = null,
				Scopes = "offline profile",
			};

			const string mockRefreshToken = "random-refresh-token";
			// TODO: Make the mocked access token into a valid JWT?
			const string mockAccessToken = "random-token";
			DateTimeOffset mockExpiryTime = DateTimeOffset.Now.AddDays(2);

			Mock<InMemoryTokenStore> tokenStoreMock = new Mock<InMemoryTokenStore>()
			{
				CallBase = true
			};
			tokenStoreMock.Setup(store => store.AddRefreshToken(It.Is(providerName, StringComparer.InvariantCultureIgnoreCase), It.Is(mockRefreshToken, StringComparer.InvariantCultureIgnoreCase))).CallBase().Verifiable(Times.Once);

			string tempOutputFileName = Path.GetTempFileName();
			try
			{
				using IHost host = SetupMocks(cliOptions =>
				{
					cliOptions.Service = providerName;
					cliOptions.OutFile = tempOutputFileName;
				}, oidcTokenOptions =>
				{
					oidcTokenOptions.Providers = new Dictionary<string, ProviderInfo>()
					{
						{
							providerName, providerInfo
						}
					};
				},
				() =>
				{
					Microsoft.Extensions.Logging.ILogger<OidcTokenClient>? logger = null;
					Mock<OidcTokenClient> tokenClientMock = new Mock<OidcTokenClient>(providerName, providerInfo, TimeSpan.FromMinutes(15) , tokenStoreMock.Object, logger!)
					{
						// call into the real OidcTokenClient unless the method has been mocked
						CallBase = true
					};

					tokenClientMock.Setup(client => client.DoLoginAsync(It.IsAny<CancellationToken>())).ReturnsAsync(() => new OidcTokenInfo() {AccessToken = mockAccessToken, RefreshToken = mockRefreshToken, TokenExpiry = mockExpiryTime} );
					return tokenClientMock.Object;
				}, tokenStoreMock.Object);

				TokenService tokenService = host.Services.GetService<TokenService>()!;
				Assert.IsNotNull(tokenService);

				await tokenService.Main();

				Assert.IsTrue(File.Exists(tempOutputFileName));
				await using FileStream fs = File.OpenRead(tempOutputFileName);
				TokenResultFile? result = JsonSerializer.Deserialize<TokenResultFile>(fs);

				Assert.AreEqual(mockAccessToken, result!.Token);
				Assert.AreEqual(mockExpiryTime, result.ExpiresAt);
				tokenStoreMock.VerifyAll();
			}
			finally
			{
				File.Delete(tempOutputFileName);
			}
		}

		[TestMethod]
		public async Task FailToAllocateToken()
		{
			const string providerName = "Test-Service";
			ProviderInfo providerInfo = new ProviderInfo()
			{
				ClientId = "test-client",
				DisplayName = "test-client",
				RedirectUri = new Uri("http://localhost:1234"),
				ServerUri = new Uri("https://path-to-idp.com"),
				Scopes = "offline profile",
				UseDiscoveryDocument = false,
			};

			Mock<InMemoryTokenStore> tokenStoreMock = new Mock<InMemoryTokenStore>()
			{
				CallBase = true
			};
			tokenStoreMock.Setup(store => store.AddRefreshToken(It.Is(providerName, StringComparer.InvariantCultureIgnoreCase), It.IsAny<string>())).CallBase().Verifiable(Times.Never);

			const string loginResultErrorMessage = "failed to login";


			string tempOutputFileName = Path.GetTempFileName();
			try
			{
				using IHost host = SetupMocks(cliOptions =>
				{
					cliOptions.Service = providerName;
					cliOptions.OutFile = tempOutputFileName;
				}, oidcTokenOptions =>
				{
					oidcTokenOptions.Providers = new Dictionary<string, ProviderInfo>()
					{
						{
							providerName, providerInfo
						}
					};
				},
				() =>
				{
					Microsoft.Extensions.Logging.ILogger<OidcTokenClient>? logger = null;
					Mock<OidcTokenClient> tokenClientMock = new Mock<OidcTokenClient>(providerName, providerInfo, TimeSpan.FromMinutes(15) , tokenStoreMock.Object, logger!)
					{
						CallBase = true
					};

					tokenClientMock.Setup(client => client.DoCreateOidcClient(It.IsAny<OidcClientOptions>())).Returns((OidcClientOptions options) =>
					{
						Mock<OidcClient> oidcClientMock = new Mock<OidcClient>(options);
						oidcClientMock.Setup(client => client.ProcessResponseAsync(It.IsAny<string>(), It.IsAny<AuthorizeState>(), It.IsAny<Parameters>(), It.IsAny<CancellationToken>())).ReturnsAsync(() => new LoginResult(loginResultErrorMessage));
						oidcClientMock.Setup(client => client.PrepareLoginAsync( It.IsAny<Parameters>(), It.IsAny<CancellationToken>())).ReturnsAsync(() => new AuthorizeState());

						return oidcClientMock.Object;
					});

					tokenClientMock.Setup(client => client.CreateHttpServer()).Returns(() =>
					{
						Mock<OidcHttpServer> httpMock = new Mock<OidcHttpServer>();

						OidcHttpRequest request = new OidcHttpRequest("", MediaTypeNames.Application.FormUrlEncoded, "POST", new MemoryStream(Encoding.Default.GetBytes("input-object")), hasBody: true, Encoding.ASCII);
						NullOidcHttpResponse response = new NullOidcHttpResponse(new MemoryStream());

						httpMock.Setup(listener => listener.ProcessNextRequestAsync()).ReturnsAsync(() => (request, response));

						return httpMock.Object;
					});
					tokenClientMock.Setup(client => client.OpenBrowser(It.IsAny<string>())).Returns(Mock.Of<Process>());
					return tokenClientMock.Object;
				}, tokenStoreMock.Object);

				TokenService tokenService = host.Services.GetService<TokenService>()!;
				Assert.IsNotNull(tokenService);

				await Assert.ThrowsExceptionAsync<LoginFailedException>(async () =>
				{
					await tokenService.Main();
				});
				
				tokenStoreMock.VerifyAll();
			}
			finally
			{
				File.Delete(tempOutputFileName);
			}
		}

		
		[TestMethod]
		public async Task SuccessfulToken()
		{
			const string providerName = "Test-Service";
			ProviderInfo providerInfo = new ProviderInfo()
			{
				ClientId = "test-client",
				DisplayName = "test-client",
				RedirectUri = new Uri("http://localhost:1234"),
				ServerUri = new Uri("https://path-to-idp.com"),
				Scopes = "offline profile",
				UseDiscoveryDocument = false,
			};

			Mock<InMemoryTokenStore> tokenStoreMock = new Mock<InMemoryTokenStore>()
			{
				CallBase = true
			};
			tokenStoreMock.Setup(store => store.AddRefreshToken(It.Is(providerName, StringComparer.InvariantCultureIgnoreCase), It.IsAny<string>())).CallBase().Verifiable(Times.Once);

			string mockAccessToken = "access-token";
			string mockRefreshToken = "refresh-token";
			DateTimeOffset mockExpiryTime = DateTimeOffset.Now.AddHours(4);

			string tempOutputFileName = Path.GetTempFileName();
			try
			{
				Mock<OidcHttpServer> httpMock = new Mock<OidcHttpServer>();
				// buffer to store the response in
				byte[] responseBuffer = new byte[16 * 1024];
				NullOidcHttpResponse response = new NullOidcHttpResponse(new MemoryStream(responseBuffer));

				using IHost host = SetupMocks(cliOptions =>
				{
					cliOptions.Service = providerName;
					cliOptions.OutFile = tempOutputFileName;
				}, oidcTokenOptions =>
				{
					oidcTokenOptions.Providers = new Dictionary<string, ProviderInfo>()
					{
						{
							providerName, providerInfo
						}
					};
				},
				() =>
				{
					Microsoft.Extensions.Logging.ILogger<OidcTokenClient>? logger = null;
					Mock<OidcTokenClient> tokenClientMock = new Mock<OidcTokenClient>(providerName, providerInfo, TimeSpan.FromMinutes(15) , tokenStoreMock.Object, logger!)
					{
						CallBase = true
					};

					tokenClientMock.Setup(client => client.DoCreateOidcClient(It.IsAny<OidcClientOptions>())).Returns((OidcClientOptions options) =>
					{
						Mock<OidcClient> oidcClientMock = new Mock<OidcClient>(options);
						Mock<LoginResult> mockLoginResult = new Mock<LoginResult>();
						mockLoginResult.Setup(m => m.RefreshToken).Returns(mockRefreshToken);
						mockLoginResult.Setup(m => m.AccessToken).Returns(mockAccessToken);
						mockLoginResult.Setup(m => m.AccessTokenExpiration).Returns(mockExpiryTime);
						oidcClientMock.Setup(client => client.ProcessResponseAsync(It.IsAny<string>(), It.IsAny<AuthorizeState>(), It.IsAny<Parameters>(), It.IsAny<CancellationToken>())).ReturnsAsync(() => mockLoginResult.Object);
						oidcClientMock.Setup(client => client.PrepareLoginAsync( It.IsAny<Parameters>(), It.IsAny<CancellationToken>())).ReturnsAsync(() => new AuthorizeState());

						return oidcClientMock.Object;
					});

					tokenClientMock.Setup(client => client.CreateHttpServer()).Returns(() =>
					{
						OidcHttpRequest request = new OidcHttpRequest("", MediaTypeNames.Application.FormUrlEncoded, "POST", new MemoryStream(Encoding.Default.GetBytes("input-object")), hasBody: true, Encoding.ASCII);
						httpMock.Setup(listener => listener.ProcessNextRequestAsync()).ReturnsAsync(() => (request, response)).Verifiable(Times.Once);

						return httpMock.Object;
					});
					tokenClientMock.Setup(client => client.OpenBrowser(It.IsAny<string>())).Returns(Mock.Of<Process>());
					return tokenClientMock.Object;
				}, tokenStoreMock.Object);

				TokenService tokenService = host.Services.GetService<TokenService>()!;
				Assert.IsNotNull(tokenService);

				await tokenService.Main();

				// check the result page to see what got written
				string s = Encoding.ASCII.GetString(responseBuffer);
				// no point in checking the exact html returned just make sure something was written
				Assert.IsFalse(string.IsNullOrEmpty(s));
				
				Assert.IsTrue(File.Exists(tempOutputFileName));
				await using FileStream fs = File.OpenRead(tempOutputFileName);
				TokenResultFile? result = JsonSerializer.Deserialize<TokenResultFile>(fs);

				// this does not actually verify that a token is correctly allocated  as that requires reaching out to a idp, and we mock away all of those parts. This can only check that the parts we do are working as such these are just the same values as we specified in the mock
				Assert.AreEqual(mockAccessToken, result!.Token);
				Assert.AreEqual(mockExpiryTime, result.ExpiresAt);

				tokenStoreMock.VerifyAll();
			}
			finally
			{
				File.Delete(tempOutputFileName);
			}
		}
	}
}