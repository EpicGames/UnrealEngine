<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <Nullable>enable</Nullable>
    <PublishDir>..\..\..\Binaries\DotNET\OidcToken\</PublishDir>
    <EnableConfigurationBindingGenerator>false</EnableConfigurationBindingGenerator>
  </PropertyGroup>

  <ItemGroup>
    <Content Include="appsettings.json">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Configuration" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Configuration.Binder" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.DependencyInjection" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Logging" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Logging.Console" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Options.DataAnnotations" Version="10.0.6" />
    <PackageReference Include="Serilog.Extensions.Hosting" Version="8.0.0" />
    <PackageReference Include="Serilog.Settings.Configuration" Version="8.0.4" />
    <PackageReference Include="Serilog.Sinks.Console" Version="6.0.0" />
    <PackageReference Include="Serilog.Sinks.File" Version="6.0.0" />
  </ItemGroup>

  <ItemGroup>
    <Compile Remove="OidcToken.Tests\**" />
    <EmbeddedResource Remove="OidcToken.Tests\**" />
    <None Remove="OidcToken.Tests\**" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\Shared\EpicGames.OIDC\EpicGames.OIDC.csproj" />
  </ItemGroup>

</Project>
