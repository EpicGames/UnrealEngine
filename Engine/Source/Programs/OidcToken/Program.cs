// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.OIDC;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net.Http;
using System.Net.Http.Json;
using System.Text.Json;
using System.Threading.Tasks;
using Serilog;
using Serilog.Events;
using System.Runtime.InteropServices;
using System.Text.Json.Serialization;

namespace OidcToken
{
	class GetHordeAuthConfigResponse
	{
		public string Method { get; set; } = String.Empty;
		public string ProfileName { get; set; } = null!;
		public string ServerUrl { get; set; } = null!;
		public string ClientId { get; set; } = null!;
		public List<string> LocalRedirectUrls { get; set; } = new List<string>();
		public string[]? Scopes { get; set; }

		public bool IsAnonymous() => Method.Equals("Anonymous", StringComparison.OrdinalIgnoreCase);
	}

	[JsonSourceGenerationOptions(PropertyNameCaseInsensitive = true)]
	[JsonSerializable(typeof(GetHordeAuthConfigResponse))]
	[JsonSerializable(typeof(Dictionary<string, object>), TypeInfoPropertyName = "SettingsDict")]
	internal partial class OidcTokenStateContext : JsonSerializerContext
	{
	}

	class Program
	{
		static async Task<int> Main(string[] args)
		{
			if (args.Any(s => s.Equals("--help") || s.Equals("-help")) || args.Length == 0)
			{
				// print help
				Console.WriteLine("Usage: OidcToken [options]");
				Console.WriteLine();
				Console.WriteLine("Options: ");
				Console.WriteLine(" --Service <serviceName> - Indicate which OIDC service you intend to connect to. The connection details of the service is configured in appsettings.json/oidc-configuration.json.");
				Console.WriteLine(" --HordeUrl <url> - Specifies the URL of a Horde server to read configuration from.");
				Console.WriteLine(" --AuthConfigUrl <url> - Specifies the URL to read auth configuration from");
				Console.WriteLine(" --ConfigPath <path> - Specify a local path to read configuration from if you do not wish to use the autodiscovery mechanism, useful if distributing oidctoken outside of a UE sync");
				Console.WriteLine(" --Mode [Query/GetToken] - Switch mode to allow you to preview operation without triggering user interaction (result can be used to determine if user interaction is required)");
				Console.WriteLine(" --OutFile <path> - Path to create json file of result");
				Console.WriteLine(" --ResultToConsole [true/false] - If true the resulting json file is output to stdout (and logs are not created)");
				Console.WriteLine(" --Unattended [true/false] - If true we assume no user is present and thus can not rely on their input");
				Console.WriteLine(" --Zen [true/false] - If true the resulting refresh token is posted to Zens token endpoints");
				Console.WriteLine(" --Project <path> - Project can be used to tell oidc token which game its working in to allow us to read game specific settings");

				return 0;
			}
			
			// disable reloadConfigOnChange in this process, as this can cause issues under wsl and we disable this for all configuration we actually load anyway
			Environment.SetEnvironmentVariable("DOTNET_hostBuilder:reloadConfigOnChange", "false");

			ConfigurationBuilder configBuilder = new();
			configBuilder.SetBasePath(AppContext.BaseDirectory)
				.AddJsonFile("appsettings.json", true, false)
				.AddCommandLine(args);

			IConfiguration config = configBuilder.Build();

			TokenServiceOptions options = new();
			config.Bind(options);

			string? profileName = null;

			GetHordeAuthConfigResponse? hordeAuthConfig = null;
			if (options.HordeUrl != null)
			{
				hordeAuthConfig = ReadHordeConfigurationAsync(options.HordeUrl).Result;
				if (hordeAuthConfig.IsAnonymous())
				{
					// Indicate to the caller that auth is disabled.
					return (int)ExitCodeHelper.ExitCode.AuthIsDisabled;
				}

				profileName = hordeAuthConfig.ProfileName;
			}

			ClientAuthConfigurationV1? remoteAuthConfig = null;
			if (options.AuthConfigUrl != null)
			{
				if (!Uri.IsWellFormedUriString(options.AuthConfigUrl, UriKind.Absolute))
				{
					throw new FormatException($"AuthConfigUrl {options.AuthConfigUrl} is not a valid url");
				}
				Uri uri = new Uri(options.AuthConfigUrl);
				try
				{
					remoteAuthConfig = await ProviderConfigurationFactory.ReadRemoteAuthConfigurationAsync(uri, options.AuthEncryptionKey);
					if (remoteAuthConfig == null)
					{
						// if we fail to read the config try the old horde path for backwards compatibility
						try
						{
							GetHordeAuthConfigResponse hordeResponse = await ReadHordeConfigurationAsync(uri);

							profileName = hordeResponse.ProfileName;
							if (String.IsNullOrEmpty(profileName))
							{
								profileName = options.AuthConfigUrl.ToString();
							}

							Dictionary<string, object> values = new Dictionary<string, object>
							{
								{ "Method", hordeResponse.Method },
								{ "ProfileName", profileName },
								{ "ServerUri", hordeResponse.ServerUrl },
								{ "ClientId", hordeResponse.ClientId },
								{ "PossibleRedirectUri", hordeResponse.LocalRedirectUrls },
								{ "Scopes", string.Join(" ", hordeResponse.Scopes ?? Array.Empty<string>()) }
							};

							remoteAuthConfig = new ClientAuthConfigurationV1()
							{
								DefaultProvider = profileName,
								Method = hordeResponse.Method,
								Providers = new Dictionary<string, ProviderInfo>()
								{
									{
										hordeResponse.ProfileName, new ProviderInfo()
										{
											ServerUri = new Uri(hordeResponse.ServerUrl),
											ClientId = hordeResponse.ClientId,
											PossibleRedirectUri = hordeResponse.LocalRedirectUrls.Select(s => new Uri(s)).ToList(),
											Scopes = string.Join(" ", hordeResponse.Scopes ?? Array.Empty<string>())
										}
									}
								}
							};
						}
						catch (InvalidDataException)
						{
							// if we still haven't been able to read the config then its format must be incorrect
							return (int)ExitCodeHelper.ExitCode.UnableToParseConfiguration;
						}
						
					}
				}
				catch (HttpRequestException)
				{
					return (int)ExitCodeHelper.ExitCode.NetworkError;
				}

				if (!string.IsNullOrEmpty(remoteAuthConfig.DefaultProvider))
				{
					profileName = remoteAuthConfig.DefaultProvider;
				}
				else if (remoteAuthConfig.Providers.Count == 1)
				{
					profileName = remoteAuthConfig.Providers.First().Key;
				}
				else
				{
					Log.Logger.Error("No DefaultProvider set from url {Url} and more then 1 provider returned, this is ambiguous, please set DefaultProvider.", options.AuthConfigUrl);
					return (int)ExitCodeHelper.ExitCode.UnableToParseConfiguration;
				}

				// Horde supports being able to know that auth is disabled, no other service has this as we always assume auth is set to oidc if oidc token is invoked.
				if (remoteAuthConfig.Method.Equals("Anonymous", StringComparison.OrdinalIgnoreCase))
				{
					// Indicate to the caller that auth is disabled.
					return (int)ExitCodeHelper.ExitCode.AuthIsDisabled;
				}
			}

			await Host.CreateDefaultBuilder(args)
				.UseSerilog((context, configuration) =>
				{
					IConfigurationSection section = context.Configuration.GetSection("Serilog");
					if (!section.GetChildren().Any())
					{
						// no serilog provider configuration, adding some defaults
						configuration.MinimumLevel.Debug();
						configuration.MinimumLevel.Override("Microsoft.Hosting.Lifetime", LogEventLevel.Warning);
					}
					configuration.ReadFrom.Configuration(context.Configuration);
					if (!options.ResultToConsole)
					{
						configuration.WriteTo.Console(restrictedToMinimumLevel: LogEventLevel.Information);
					}

					// configure logging output directory match expectation per platform
					if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
					{
						configuration.WriteTo.File(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "UnrealEngine\\Common\\OidcToken\\Logs\\oidc-token.log"), rollingInterval:RollingInterval.Day, restrictedToMinimumLevel: LogEventLevel.Debug, retainedFileCountLimit: 7, shared: true);
					}
					else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
					{
						configuration.WriteTo.File(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData), "Epic/UnrealEngine/Common/OidcToken/Logs/oidc-token.log"), rollingInterval: RollingInterval.Day, restrictedToMinimumLevel: LogEventLevel.Debug, retainedFileCountLimit: 7, shared: true);
					}
					else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
					{
						configuration.WriteTo.File(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData), "Epic/UnrealEngine/Common/OidcToken/Logs/oidc-token.log"), rollingInterval: RollingInterval.Day, restrictedToMinimumLevel: LogEventLevel.Debug, retainedFileCountLimit: 7, shared: true);
					}
				})
				.ConfigureAppConfiguration(builder =>
				{
					builder.AddConfiguration(config);
					
					if (string.IsNullOrEmpty(options.Service) && !string.IsNullOrEmpty(profileName))
					{
						Dictionary<string, string?> values = new Dictionary<string, string?>
						{
							[nameof(TokenServiceOptions.Service)] = profileName
						};
						builder.AddInMemoryCollection(values);
					}
				})
				.ConfigureServices(
				(content, services) =>
				{
					IConfiguration configuration = content.Configuration;
					services.AddOptions<TokenServiceOptions>().Bind(configuration).ValidateDataAnnotations();

					IConfiguration serviceConfig;
					if (hordeAuthConfig != null)
					{
						Dictionary<string, string?> values = new Dictionary<string, string?>();
						values[$"{nameof(OidcTokenOptions.Providers)}:{hordeAuthConfig.ProfileName}:{nameof(ProviderInfo.DisplayName)}"] = hordeAuthConfig.ProfileName;
						values[$"{nameof(OidcTokenOptions.Providers)}:{hordeAuthConfig.ProfileName}:{nameof(ProviderInfo.ServerUri)}"] = hordeAuthConfig.ServerUrl;
						values[$"{nameof(OidcTokenOptions.Providers)}:{hordeAuthConfig.ProfileName}:{nameof(ProviderInfo.ClientId)}"] = hordeAuthConfig.ClientId;
						values[$"{nameof(OidcTokenOptions.Providers)}:{hordeAuthConfig.ProfileName}:{nameof(ProviderInfo.RedirectUri)}"] = hordeAuthConfig.LocalRedirectUrls![0];
						if (hordeAuthConfig.Scopes is { Length: > 0 })
						{
							values[$"{nameof(OidcTokenOptions.Providers)}:{hordeAuthConfig.ProfileName}:{nameof(ProviderInfo.Scopes)}"] = String.Join(" ", hordeAuthConfig.Scopes);
						}
						serviceConfig = new ConfigurationBuilder().AddInMemoryCollection(values).Build();
					}
					else if (remoteAuthConfig != null)
					{
						serviceConfig = ProviderConfigurationFactory.BindOptions(remoteAuthConfig);
					}
					else if (options.ConfigPath != null && File.Exists(options.ConfigPath))
					{
						ConfigurationBuilder oidcConfigurationBuilder = new ConfigurationBuilder();
						oidcConfigurationBuilder.AddJsonFile(options.ConfigPath, false, false);
						IConfiguration oidcConfig = oidcConfigurationBuilder.Build();
						serviceConfig = oidcConfig.GetSection("OidcToken");
					}
					else
					{
						// guess where the engine directory is based on the assumption that we are running out of Engine\Binaries\DotNET\OidcToken\<platform>
						DirectoryInfo engineDir = new DirectoryInfo(Path.Combine(AppContext.BaseDirectory, "../../../../../Engine"));
						if (!engineDir.Exists)
						{
							// try to see if engine dir can be found from the current code path Engine\Source\Programs\OidcToken\bin\<Configuration>\<.net-version>
							engineDir = new DirectoryInfo(Path.Combine(AppContext.BaseDirectory, "../../../../../../../Engine"));

							if (!engineDir.Exists)
							{
								throw new Exception($"Unable to guess engine directory so unable to continue running. Starting directory was: {AppContext.BaseDirectory}");
							}
						}

						serviceConfig = ProviderConfigurationFactory.ReadConfiguration(engineDir, !string.IsNullOrEmpty(options.Project) ? new DirectoryInfo(options.Project) : null);
					}
					services.AddOptions<OidcTokenOptions>().Bind(serviceConfig).ValidateDataAnnotations();

					services.AddSingleton<IOidcTokenManager, OidcTokenManager>();
					services.AddSingleton<IOidcTokenClientFactory, OidcTokenClientFactory>();
					services.AddSingleton<ITokenStore>(TokenStoreFactory.CreateTokenStore);

					services.AddHostedService<TokenService>();
				})
				.RunConsoleAsync();

			return Environment.ExitCode;
		}

		static async Task<GetHordeAuthConfigResponse> ReadHordeConfigurationAsync(Uri hordeUrl)
		{
			// Read the configuration settings from the Horde server
			GetHordeAuthConfigResponse? authConfig;
			using (HttpClient httpClient = new HttpClient())
			{
				using HttpRequestMessage request = new HttpRequestMessage(HttpMethod.Get, new Uri(hordeUrl, "api/v1/server/auth"));
				using HttpResponseMessage response = await httpClient.SendAsync(request);
				response.EnsureSuccessStatusCode();

				authConfig = await response.Content.ReadFromJsonAsync<GetHordeAuthConfigResponse>(OidcTokenStateContext.Default.GetHordeAuthConfigResponse);

				if (authConfig == null)
				{
					throw new InvalidDataException("Server returned an empty auth config object");
				}
			}

			if (!authConfig.IsAnonymous())
			{
				string? localRedirectUrl = authConfig.LocalRedirectUrls.FirstOrDefault();
				if (String.IsNullOrEmpty(authConfig.ServerUrl) || String.IsNullOrEmpty(authConfig.ClientId) || String.IsNullOrEmpty(localRedirectUrl))
				{
					throw new Exception("No auth server configuration found");
				}

				if (String.IsNullOrEmpty(authConfig.ProfileName))
				{
					authConfig.ProfileName = hordeUrl.Host.ToString();
				}
			}

			return authConfig;
		}
	}

	static class ExitCodeHelper
	{
		public enum ExitCode
		{
			/// <summary>
			/// Successful run
			/// </summary>
			Success = 0,
			/// <summary>
			/// Unknown error
			/// </summary>
			UnknownError = 1,
			/// <summary>
			/// Unable to start the http server, mostly likely a port collision
			/// </summary>
			UnableToStartHttpServer = 2,
			/// <summary>
			/// Network error, likely caused by bad or lack of network connection. Includes DNS resolve issues.
			/// </summary>
			NetworkError = 3,
			/// <summary>
			/// Successfully fetched a remote configuration but there were some parsing or logic issue with it
			/// </summary>
			UnableToParseConfiguration = 9,
			/// <summary>
			/// Something went wrong when trying to allocate a new access token. Most often the refresh token is expired, and we are running with --unattended so we can not trigger a new interactive login.
			/// </summary>
			UnableToAllocateToken = 10,
			/// <summary>	
			/// Auth is disabled according to the remote configuration
			/// </summary>
			AuthIsDisabled= 11,
		}
	}
}