<?xml version="1.0" encoding="utf-8"?>
<!--
https://go.microsoft.com/fwlink/?LinkID=208121.
-->
<Project>
  <PropertyGroup>
    <Configuration>Release</Configuration>
    <Platform>Any CPU</Platform>
    <PublishDir>..\..\..\Binaries\DotNET\OidcToken\osx-x64\</PublishDir>
    <PublishProtocol>FileSystem</PublishProtocol>
    <TargetFramework>net8.0</TargetFramework>
    <RuntimeIdentifier>osx-x64</RuntimeIdentifier>
    <SelfContained>true</SelfContained>
    <PublishSingleFile>true</PublishSingleFile>
    <PublishTrimmed>true</PublishTrimmed>
    <InvariantGlobalization>true</InvariantGlobalization>
  </PropertyGroup>
</Project>