<?xml version="1.0" encoding="utf-8"?>
<!--
https://go.microsoft.com/fwlink/?LinkID=208121.
-->
<Project>
  <PropertyGroup>
    <History>True|2022-06-29T11:06:41.3153976Z;True|2022-06-29T11:04:38.4946263+02:00;True|2022-06-29T10:52:37.1295841+02:00;True|2022-06-28T16:01:36.6091443+02:00;True|2022-06-28T15:58:35.7907117+02:00;True|2022-06-23T16:12:34.0409814+02:00;True|2022-06-22T17:51:37.8445736+02:00;</History>
  </PropertyGroup>
</Project>