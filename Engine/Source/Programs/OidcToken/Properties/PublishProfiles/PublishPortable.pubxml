<?xml version="1.0" encoding="utf-8"?>
<!--
https://go.microsoft.com/fwlink/?LinkID=208121.
-->
<Project>
  <PropertyGroup>
    <Configuration>Release</Configuration>
    <Platform>Any CPU</Platform>
    <PublishDir>..\..\..\Binaries\DotNET\OidcToken\portable\</PublishDir>
    <PublishProtocol>FileSystem</PublishProtocol>
    <TargetFramework>net8.0</TargetFramework>
    <SelfContained>false</SelfContained>
    <InvariantGlobalization>true</InvariantGlobalization>
  </PropertyGroup>
</Project>