{
  "profiles": {
    "oidc_token": {
      "commandName": "Project",
      "commandLineArgs": "--OutFile=results.json"
    }
  }
}