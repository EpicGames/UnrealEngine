// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.IO;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.OIDC;
using System.Diagnostics;

namespace OidcToken
{
	[JsonSerializable(typeof(ZenAddOidcTokenRequest))]
	[JsonSerializable(typeof(TokenResultFile))]
	[JsonSerializable(typeof(TokenStatusFile))]
	internal partial class TokenServiceStateContext : JsonSerializerContext
	{
	}

	internal class TokenResultFile
	{
		public string Token { get; set; }
		public DateTimeOffset ExpiresAt { get; set; }
		public DateTime ExpiresAtUtc { get; set; }

		public TokenResultFile(string token, DateTimeOffset expiresAt)
		{
			this.Token = token;
			this.ExpiresAt = expiresAt;
			this.ExpiresAtUtc = expiresAt.DateTime.ToUniversalTime();
		}
	}

	internal class TokenStatusFile
	{
		public string Service { get; set; }

		public OidcStatus Status { get; set; }

		public TokenStatusFile(string service, OidcStatus status)
		{
			this.Service = service;
			this.Status = status;
		}
	}


	internal class ZenAddOidcTokenRequest
	{
		public string? ProviderName { get; set; }
		public string? RefreshToken { get; set; }
	}

	internal class TokenService : IHostedService
	{
		private readonly ILogger<TokenService> Logger;
		private readonly IHostApplicationLifetime AppLifetime;
		private readonly IOptionsMonitor<TokenServiceOptions> Settings;
		private readonly IOidcTokenManager TokenManager;
		private int? ExitCode;

		public TokenService(ILogger<TokenService> logger, IHostApplicationLifetime appLifetime, IOptionsMonitor<TokenServiceOptions> settings, IOidcTokenManager tokenManager)
		{
			this.Logger = logger;
			this.AppLifetime = appLifetime;
			this.Settings = settings;
			this.TokenManager = tokenManager;
		}

		public Task StartAsync(CancellationToken cancellationToken)
		{
			AppLifetime.ApplicationStarted.Register(async () =>
			{
				try
				{
					Logger.LogDebug("OidcToken started with arguments: {@Settings}", Settings.CurrentValue);
					await Main();
					ExitCode = (int)ExitCodeHelper.ExitCode.Success;
				}
				catch (UnableToAllocateTokenException)
				{
					Logger.LogWarning("Was unable to allocate a token");
					ExitCode = (int)ExitCodeHelper.ExitCode.UnableToAllocateToken;
				}
				catch (HttpServerException e)
				{
					Logger.LogWarning("Unable to start http server:" + e.Message);
					ExitCode = (int)ExitCodeHelper.ExitCode.UnableToStartHttpServer;
				}
				catch (Exception ex)
				{
					Logger.LogError(ex, "Unhandled exception!");
					ExitCode = (int)ExitCodeHelper.ExitCode.UnknownError;
				}
				finally
				{
					// Stop the application once the work is done
					AppLifetime.StopApplication();
				}
			});

			return Task.CompletedTask;
		}

		public async Task Main()
		{
			string providerName = Settings.CurrentValue.Service;

			switch (Settings.CurrentValue.Mode)
			{
				case TokenServiceOptions.TokenServiceMode.Query:
				{
					await OutputStatus(providerName, await TokenManager.GetStatusForProvider(providerName));
					break;
				}
				case TokenServiceOptions.TokenServiceMode.GetToken:
				{
					OidcStatus status = await TokenManager.GetStatusForProvider(providerName);
					Logger.LogInformation("Determined status of provider {ProviderName} was {Status}", providerName, status);

					IOidcTokenInfo? tokenInfo;
					if (status == OidcStatus.NotLoggedIn && !Settings.CurrentValue.Unattended)
					{
						Logger.LogInformation("Logging in to provider {ProviderName}", providerName);
						tokenInfo = await TokenManager.LoginAsync(providerName);
					}
					else
					{
						Logger.LogInformation("Fetching access token from provider {ProviderName}", providerName);

						try
						{
							tokenInfo = await TokenManager.GetAccessToken(providerName);
						}
						catch (NotLoggedInException)
						{
							if (Settings.CurrentValue.Unattended)
							{
								Logger.LogWarning("Not logged in to provider {ProviderName} but was running unattended so unable to login", providerName);
								throw new UnableToAllocateTokenException();
							}
							else
							{
								Logger.LogInformation("Logging in to provider {ProviderName}", providerName);
								tokenInfo = await TokenManager.LoginAsync(providerName);
							}
						}
					}

					if (!tokenInfo.IsValid(DateTimeOffset.Now))
					{
						throw new Exception("Failed to allocate a token");
					}

					if (Settings.CurrentValue.ResultToConsole)
					{
						string s = JsonSerializer.Serialize(new TokenResultFile(tokenInfo.AccessToken!, tokenInfo.TokenExpiry), TokenServiceStateContext.Default.TokenResultFile);
						Console.WriteLine(s);
					}

					if (!string.IsNullOrEmpty(Settings.CurrentValue.OutFile))
					{
						Logger.LogInformation("Token resolved, outputting result to '{OutFile}'", Settings.CurrentValue.OutFile);
						await OutputToken(tokenInfo.AccessToken!, tokenInfo.TokenExpiry);
					}

					if (Settings.CurrentValue.Zen)
					{
						Logger.LogInformation("Saving token to Zen instance '{ZenUrl}'", Settings.CurrentValue.ZenUrl);
						
						try
						{
							using HttpClient client = new();
							client.DefaultRequestHeaders.Accept.Add(new MediaTypeWithQualityHeaderValue("application/json"));

							string zenUrl = $"{Settings.CurrentValue.ZenUrl}/auth/oidc/refreshtoken";
							ZenAddOidcTokenRequest request = new ZenAddOidcTokenRequest { ProviderName = providerName, RefreshToken = tokenInfo.RefreshToken };
							HttpContent content = new StringContent(JsonSerializer.Serialize<ZenAddOidcTokenRequest>(request, TokenServiceStateContext.Default.ZenAddOidcTokenRequest), Encoding.UTF8, "application/json");
							HttpResponseMessage result = await client.PostAsync(zenUrl, content);
								
							if (result.IsSuccessStatusCode)
							{
								Logger.LogInformation("Successfully stored token to Zen");
							}
							else
							{
								Logger.LogInformation("Failed to store token to Zen");
							}
						}
						catch (Exception err)
						{
							Logger.LogInformation("An error occurred saving token to Zen, reason '{Reason}'", err.Message);
							throw new Exception("Failed to store token to Zen", err);
						}
					}
					break;
				}
				default:
					throw new NotImplementedException();
			}
		}

		private async Task OutputToken(string token, DateTimeOffset expiresAt)
		{
			FileInfo fi = new(Settings.CurrentValue.OutFile);

			if (fi.DirectoryName != null)
			{
				Directory.CreateDirectory(fi.DirectoryName);
			}

			Logger.LogInformation("Token output to \"{OutFile}\"", fi.FullName);

			await using FileStream fs = fi.Open(FileMode.Create, FileAccess.Write);
			await JsonSerializer.SerializeAsync(fs, new TokenResultFile(token, expiresAt), TokenServiceStateContext.Default.TokenResultFile);
		}

		private async Task OutputStatus(string service, OidcStatus status)
		{
			if (Settings.CurrentValue.ResultToConsole)
			{
				string s = JsonSerializer.Serialize(new TokenStatusFile(service, status), TokenServiceStateContext.Default.TokenStatusFile);
				Console.WriteLine(s);
			}

			if (status == OidcStatus.NotLoggedIn)
			{
				Logger.LogWarning("Token for provider {ProviderName} does not exist or is old.", service);
			}
			else
			{
				// verify that the refresh token is valid and actually able to generate a access token
				await TokenManager.GetAccessToken(service);
				OidcStatus refreshedStatus = await TokenManager.GetStatusForProvider(service);
				Logger.LogInformation("Determined status of provider {ProviderName} was {Status}", service, refreshedStatus);
			}

			if (!string.IsNullOrEmpty(Settings.CurrentValue.OutFile))
			{
				FileInfo fi = new(Settings.CurrentValue.OutFile);
				Logger.LogInformation("Token output to \"{OutFile}\"", fi.FullName);

				await using FileStream fs = fi.Open(FileMode.Create, FileAccess.Write);
				await JsonSerializer.SerializeAsync<TokenStatusFile>(fs, new TokenStatusFile(service, status), TokenServiceStateContext.Default.TokenStatusFile);
			}
		}

		public Task StopAsync(CancellationToken cancellationToken)
		{
			Environment.ExitCode = ExitCode.GetValueOrDefault(-1);
			return Task.CompletedTask;
		}
	}

	internal class UnableToAllocateTokenException : Exception
	{
	}

	internal class TokenServiceOptions: IValidatableObject
	{
		public enum TokenServiceMode
		{
			Query,
			GetToken,
		}

		/// <summary>
		/// The provider identifier you wish to login to
		/// </summary>
		[Required] public string Service { get; set; } = null!;

		/// <summary>
		/// URL of the horde server to read provider information from
		/// </summary>
		public Uri? HordeUrl { get; set; }

		/// <summary>
		/// URL of the server to read provider information from
		/// </summary>
		public string? AuthConfigUrl { get; set; }

		/// <summary>
		/// The encryption key used for the remote auth, the default key is usually used to avoid having to ship this config value. This needs to be a 16 byte string hex encoded string.
		/// </summary>
		public string AuthEncryptionKey { get; set; } = ProviderConfigurationFactory.DefaultEncryptionKey;

		/// <summary>
		/// The mode we are running OidcToken in
		/// </summary>
		[Required] public TokenServiceMode Mode { get; set; } = TokenServiceMode.GetToken;

		/// <summary>
		/// Set to output results to stdout (can be combined with a file)
		/// </summary>
		public bool ResultToConsole { get; set; } = false;

		/// <summary>
		/// A path to were we output a file with our results, format will depend on mode. Set to empty string to disable.
		/// </summary>
		public string OutFile { get; set; } = null!;

		/// <summary>
		/// If set this indicates we should not expect a user to be present
		/// </summary>
		public bool Unattended { get; set; } = false;

		/// <summary>
		/// If set this indicates a zen instance exists and we should share the token with it
		/// </summary>
		public bool Zen { get; set; } = false;

		/// <summary>
		/// The url to the zen server that we should share a token with (if Zen flag is set)
		/// </summary>
		public string ZenUrl { get; set; } = "http://localhost:8558";
		
		/// <summary>
		/// Path to the game root directory
		/// </summary>
		public string? Project { get; set; } = string.Empty;

		/// <summary>
		/// Path to the configuration file to use if you know exactly which config to use
		/// </summary>
		public string? ConfigPath { get; set; } = null;

		public IEnumerable<ValidationResult> Validate(ValidationContext validationContext)
		{
			List<ValidationResult> results = new List<ValidationResult>();

			if (AuthConfigUrl == null && HordeUrl == null)
			{
				Validator.TryValidateProperty(Service, new ValidationContext(this, null, null) { MemberName = nameof(Service) }, results);
			}
			return results;
		}
	}
}
