{
	"Serilog": {
		"Using": [ "Serilog.Sinks.File", "Serilog.Sinks.Console" ],
		"MinimumLevel": {
			"Default": "Debug",
			"Override": {
				"Microsoft.Hosting.Lifetime": "Warning"
			}
		}
	}
}
