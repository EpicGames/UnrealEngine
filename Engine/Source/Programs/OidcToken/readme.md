OidcToken is a small tool to help expose the ability to allocate access and refresh tokens from a Oidc compatible Identity Provider.

# Configuration
Configuration file can be placed under 
`Engine\Programs\OidcToken\oidc-configuration.json`
or `<Game>\Programs\OidcToken\oidc-configuration.json`

The configuration file can look like this:
```
{
	"OidcToken": {
		"Providers": {
			"MyOwnProvider": {
				"ServerUri": "https://<url-to-your-provider>",
				"ClientId": "<unique-id-from-provider-usually-guid>",
				"DisplayName": "MyOwnProvider",
				"RedirectUri": "http://localhost:6556/callback", // this needs to match what is configured as the redirect uri for your IdP
				"PossibleRedirectUri": [
					"http://localhost:6556/callback",
					"http://localhost:6557/callback",
				], // set of redirect uris that can be used, ports can be in use so it is a good idea to configure a few alternatives. these needs to match configuration in IdP
				"Scopes": "openid profile offline_access", // these scopes are the basic ones you will need, some system may require more or less and they may be named differently
				"GenericErrorInformation": "An extra message that is displayed during errors, can be used to describe specific processes or contact information for support"
			}
		}
	}
}
```

# Setting up the Idenity Provider
Here follows a attempt at documenting how you can setup some common Identity Providers (IdP) - OIDC is a standard and as such this is no way a list of softwares you have to use rather this is provided as a way to help with setup for common choices.

## Okta

You will need a few things setup in Okta: A Client (Application) for interactive login (by users), a Custom Auth Server for control over how okta maps claims and scopes and some groups that you can assign users to manage access control, lastly if you want to run this in a buildfarm were interactive login is not possible you will need another Client (Application) setup to allow for logins.
Do note that custom auth server is a extra feature in Okta that you will need to have, otherwise Okta does not support OIDC logins.

First of we create the Client for interactive login under Applications -> Applications, you will want to allow "Refresh Tokens" and "Authorization Code" grant types. For sign in urls we recommended specifying a few of them under localhost, something like this:
* http://localhost:8749/oidc-token
* http://localhost:8750/oidc-token
* http://localhost:8751/oidc-token
* http://localhost:8752/oidc-token
* http://localhost:8753/oidc-token
* http://localhost:8754/oidc-token
This allows the app to run on multiple local ports during the login process which avoids issues with the port being busy. (the ports are arbitrary)

Next we setup the unattended login client, which is similar to the first one except you want the "Client Credentials" grant type instead (and no need for for urls). This is a bit tricky to setup in Okta and will require you to use profile objects with your client, see https://developer.okta.com/docs/reference/api/apps/#update-application-level-profile-attributes for examples on how to update the created profile.
Once you have created the client credential you want to use for unattended logins you will need to submit a payload like this for it:
```
"profile": {
        "clientCredentialsGroups": [
            "app-ue-storage-project-your-project-name"
        ]
    }
```
Also make sure to update your groups claim in the custom auth server as described below.

As for the groups for users you can map these however you want so you just need a group that accurately includes all users you want to have access, we usually have one per project. You will also want an admin group for users that have some added access. It can be good to have a naming convention that makes it easy to identity the groups you want to use (to make sure we only send the groups that apply to Unreal things as part of the tokens and not all groups a user belongs to). Make sure to assign at least one user to this group for now.

For the custom auth server you will find that under Security -> API in the Okta admin page, note that this is an addon to Okta that may not be available for you but unfortunately is required for Okta to be able to handle OIDC logins. You create the auth server by clicking the "Create Authorization Server" button this auth server is not Cloud DDC specific but rather something you can use for any Unreal Engine services you will want to use. Next edit the auth server and setup "Access Policies", you will want one policy per client created and just allow that client to login. 

Next go to "Claims" for the Access token type and setup a "groups" claim and set it to filter out the groups which you actually want to use as well as include the `clientCredentialsGroups` using this custom expression: (this filters out groups starting with `app-ue`)
```(appuser != null) ? Arrays.flatten( Groups.startsWith("OKTA", "app-ue-", 100) == null ? {} : Groups.startsWith("OKTA", "app-ue-", 100) ) : app.profile.clientCredentialsGroups```
You will also need to setup a "cache_access" scope which we can use for the build farm logins.
That should be everything you need in Okta, to test this you can go to the "Token Preview" section in the auth server and choose the interactive login client with "Authorization Code" grant type and select the user which is assigned to the correct group. When you preview this token the json it shows should include a "groups" array which includes the names of the groups you have assigned

## Microsoft Entra (AzureAD)

Go to `App registration` and create a new app registration for the desktop logins. It should be single tenant and contain a set of localhost redirect uris, something like this (use the public client/native option):
* http://localhost:8749/oidc-token
* http://localhost:8750/oidc-token
* http://localhost:8751/oidc-token
* http://localhost:8752/oidc-token
* http://localhost:8753/oidc-token
* http://localhost:8754/oidc-token

Note down the `client id` of this app.

After this has been created you should go to `Token Configuration` and add a `groups` claim setting to use `Groups assigned to the application`.
Create a new security group for the project user role, assign that security group to the role and then add all users you want to the security group.
Create the app roles for the project user (usually one per project) and the admin role, the project user role needs to be assignable to both users/groups and applications while the admin role is only for users.

Go back to `App registration` and create a app for the backend service (`Unreal Cloud DDC` for example), add a API scope to this app calling it `user.access`. Assign the `client id` of the desktop app access to this api. 

Create a new separate app registration for your cooking apps and add a client secret to them so they can login unattended (or use managed identity or similar if you prefer), this app should also be assigned the project user role.

When creating the oidc-configuration.json you can find the server uri to use under the `Endpoints` button for your app registration, its usually https://login.microsoftonline.com/<directory-tenant-id>/v2.0
For Client id use the `client id` of the desktop app you created.
The scope needs to contain the API scope you created in the backend service, so it usually ends up being: `offline_access profile openid api://<api scope guid>/user.access`

## Google

Unfortunatley at this time Google does not support the features we require to able to use them as a auth source directly. Specifically we rely on JWTs for API access control, each oidc provider tends to call this slightly different things but in practice it means we can rely on the IdP to provide access token for individual resources using a JWT token. Instead Google only provide for a OIDC api that enables us to interact with their users information and federate the login which would force us to simply sync the users into our own user management system and provide the tokens from there. This is currently out of scope from how we use these tokens.

A second problem that prevents us from working around this issue is Googles lack of groups in the id tokens which is tracked here:
https://issuetracker.google.com/issues/133774835

For workarounds there are identity providers (IdPs) that can be used to federate the Google login and then manage api access tokens as we require. Some licensees have used used `Dex` for this. We have used `Identity Server 4` for similar features in the past (before it was discontinued). There are multiple open source IdPs that could likely serve this purpose.
 
There maybe other solutions out there, we have very limited experience with Googles auth so if someone finds a secure and easy way to interact with the their auth we would be happy to accept changes for this.